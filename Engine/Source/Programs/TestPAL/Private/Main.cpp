// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestPALLog.h"
#include "Parent.h"
#include "Logging/StructuredLog.h"
#include "Misc/Guid.h"
#include "Stats/StatsMisc.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformMemory.h"
#include "HAL/PlatformApplicationMisc.h"
#include "GenericPlatform/GenericApplication.h"

#include "TestDirectoryWatcher.h"
#include "RequiredProgramMainCPPInclude.h"
#include "HAL/MallocPoisonProxy.h"
#include "HAL/ThreadSafeCounter64.h"

#include "Misc/Fork.h"
#include "Async/MappedFileHandle.h"

DEFINE_LOG_CATEGORY(LogTestPAL);

IMPLEMENT_APPLICATION(TestPAL, "TestPAL");

#define ARG_PROC_TEST						"proc"
#define ARG_PROC_TEST_CHILD					"proc-child"
#define ARG_PROC_KILLTREE_TEST				"proc-killtree"
#define ARG_PROC_KILLTREE_TEST_CHILD		"proc-killtree-child"
#define ARG_PROC_KILLTREE_TEST_GRANDCHILD	"proc-killtree-grandchild"
#define ARG_PROC_OPENPROCESS_TEST			"proc-openprocess"
#define ARG_CASE_SENSITIVITY_TEST			"case"
#define ARG_MESSAGEBOX_TEST					"messagebox"
#define ARG_DIRECTORY_WATCHER_TEST			"dirwatcher"
#define ARG_THREAD_SINGLETON_TEST			"threadsingleton"
#define ARG_SYSINFO_TEST					"sysinfo"
#define ARG_CRASH_TEST						"crash"
#define ARG_ENSURE_TEST						"ensure"
#define ARG_STRINGPRECISION_TEST			"stringprecision"
#define ARG_DSO_TEST						"dso"
#define ARG_GET_ALLOCATION_SIZE_TEST		"getallocationsize"
#define ARG_MALLOC_THREADING_TEST			"mallocthreadtest"
#define ARG_MALLOC_REPLAY					"mallocreplay"
#define ARG_THREAD_PRIO_TEST				"threadpriotest"
#define ARG_INLINE_CALLSTACK_TEST			"inline"
#define ARG_STRINGS_ALLOCATION_TEST			"stringsallocation"
#define ARG_CREATEGUID_TEST					"createguid"
#define ARG_THREADSTACK_TEST				"threadstack"
#define ARG_EXEC_PROCESS_TEST				"exec-process"
#define ARG_FORK_TEST						"fork"
#define ARG_CMDLINE_PARSE_TEST				"cmdline"
// Test that it is possible to replace a file open (for reading) in one Unreal process by another
// Consists of two parts
//    file-keep-open  - the instance that will open the file in one instance and will keep it so
//    file-replace-open - the instance that will attempt to replace
#define ARG_CMDLINE_FILE_KEEP_OPEN			"file-keep-open"
#define ARG_CMDLINE_FILE_REPLACE_OPEN		"file-replace-open"
#define ARG_CMDLINE_FILE_DELETE_MAPPED		"file-delete-mapped"	// self-contained test, opens a mapped file and tries to delete it

namespace TestPAL
{
	FString CommandLine;
};

/**
 * FProcHandle test (child instance)
 */
int32 ProcRunAsChild(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	// set a random delay pretending to do some useful work up to a minute. 
	srand(FPlatformProcess::GetCurrentProcessId());
	double RandomWorkTime = FMath::FRandRange(0.0f, 6.0f);

	UE_LOGF(LogTestPAL, Display, "Running proc test as child (pid %d), will be doing work for %f seconds.", FPlatformProcess::GetCurrentProcessId(), RandomWorkTime);

	double StartTime = FPlatformTime::Seconds();

	// Use all the CPU!
	for (;;)
	{
		double CurrentTime = FPlatformTime::Seconds();
		if (CurrentTime - StartTime >= RandomWorkTime)
		{
			break;
		}
	}

	UE_LOGF(LogTestPAL, Display, "Child (pid %d) finished work.", FPlatformProcess::GetCurrentProcessId());

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * FProcHandle test (parent instance)
 */
int32 ProcRunAsParent(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running proc test as parent.");

	// Run child instance continuously
	int NumChildrenToSpawn = 255, MaxAtOnce = 5;
	FParent Parent(NumChildrenToSpawn, MaxAtOnce);

	Parent.Run();

	UE_LOGF(LogTestPAL, Display, "Parent quit.");

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * TerminateProc KillTree test (grandchild instance).
 * Spins until signalled. Installed as a leaf process in the tree under proc-killtree-child.
 */
int32 ProcKillTreeRunAsGrandchild(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running proc-killtree test as grandchild (pid %d), spinning until killed.",
		FPlatformProcess::GetCurrentProcessId());

	while (!IsEngineExitRequested())
	{
		FPlatformProcess::Sleep(0.25f);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * TerminateProc KillTree test (child instance).
 * Spawns a grandchild, writes the grandchild's PID to the file specified by -pidfile=, then spins until signalled.
 */
int32 ProcKillTreeRunAsChild(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	FString PidFilePath;
	if (!FParse::Value(CommandLine, TEXT("pidfile="), PidFilePath))
	{
		UE_LOGF(LogTestPAL, Error, "proc-killtree-child requires a -pidfile=<path> argument.");
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "Running proc-killtree test as child (pid %d), spawning grandchild.",
		FPlatformProcess::GetCurrentProcessId());

	FString WorkerName = FPlatformProcess::ExecutablePath();
	uint32 GrandchildPid = 0;
	FProcHandle GrandchildHandle = FPlatformProcess::CreateProc(*WorkerName, TEXT(ARG_PROC_KILLTREE_TEST_GRANDCHILD),
		false, false, false, &GrandchildPid, -1, nullptr, nullptr);

	if (!GrandchildHandle.IsValid())
	{
		UE_LOGF(LogTestPAL, Error, "proc-killtree-child: failed to spawn grandchild.");
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "Child (pid %d) spawned grandchild (pid %u), writing PID to '%ls'.",
		FPlatformProcess::GetCurrentProcessId(), GrandchildPid, *PidFilePath);

	// Write the grandchild PID to the agreed-upon file so the parent can verify it was killed
	if (!FFileHelper::SaveStringToFile(FString::Printf(TEXT("%u"), GrandchildPid), *PidFilePath))
	{
		UE_LOGF(LogTestPAL, Error, "proc-killtree-child: failed to write PID file '%ls', terminating grandchild.", *PidFilePath);
		FPlatformProcess::TerminateProc(GrandchildHandle, true);
		FPlatformProcess::WaitForProc(GrandchildHandle);
		FPlatformProcess::CloseProc(GrandchildHandle);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	// Spin until the kill tree signal arrives
	while (!IsEngineExitRequested())
	{
		FPlatformProcess::Sleep(0.25f);
	}

	FPlatformProcess::WaitForProc(GrandchildHandle);
	FPlatformProcess::CloseProc(GrandchildHandle);

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * TerminateProc KillTree test (parent / driver instance).
 * Spawns a child which itself spawns a grandchild, then verifies that calling
 * TerminateProc(..., KillTree=true) kills both.
 */
int32 ProcKillTreeRunAsParent(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running proc-killtree test as parent (pid %d).",
		FPlatformProcess::GetCurrentProcessId());

	// Unique PID file the child will write its grandchild's PID into
	FString PidFilePath = FPaths::Combine(
		FPlatformProcess::UserTempDir(),
		FString::Printf(TEXT("testpal-killtree-%u.pid"), FPlatformProcess::GetCurrentProcessId()));

	// Spawn the child, forwarding the PID file path
	FString WorkerName = FPlatformProcess::ExecutablePath();
	FString ChildArgs = FString::Printf(TEXT(ARG_PROC_KILLTREE_TEST_CHILD) TEXT(" -pidfile=\"%s\""), *PidFilePath);
	uint32 ChildPid = 0;
	FProcHandle ChildHandle = FPlatformProcess::CreateProc(*WorkerName, *ChildArgs,
		false, false, false, &ChildPid, -1, nullptr, nullptr);

	if (!ChildHandle.IsValid())
	{
		UE_LOGF(LogTestPAL, Error, "Failed to spawn child process.");
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "Spawned child (pid %u). Waiting for grandchild PID file '%ls'.",
		ChildPid, *PidFilePath);

	// Poll for the PID file written by the child once its grandchild is running
	const double PidFileTimeoutSeconds = 15.0;
	double StartTime = FPlatformTime::Seconds();
	while (!FPaths::FileExists(PidFilePath))
	{
		if (FPlatformTime::Seconds() - StartTime > PidFileTimeoutSeconds)
		{
			UE_LOGF(LogTestPAL, Error, "Timed out waiting for grandchild PID file.");
			FPlatformProcess::TerminateProc(ChildHandle, true);
			FPlatformProcess::WaitForProc(ChildHandle);
			FPlatformProcess::CloseProc(ChildHandle);
			IFileManager::Get().Delete(*PidFilePath);
			FEngineLoop::AppPreExit();
			FEngineLoop::AppExit();
			return 1;
		}
		FPlatformProcess::Sleep(0.05f);
	}

	// Brief pause to let the child finish writing and close the file handle;
	// FileExists can return true before the write is fully flushed.
	FPlatformProcess::Sleep(0.25f);

	FString GrandchildPidStr;
	if (!FFileHelper::LoadFileToString(GrandchildPidStr, *PidFilePath))
	{
		UE_LOGF(LogTestPAL, Error, "Failed to read grandchild PID file '%ls'.", *PidFilePath);
		FPlatformProcess::TerminateProc(ChildHandle, true);
		FPlatformProcess::WaitForProc(ChildHandle);
		FPlatformProcess::CloseProc(ChildHandle);
		IFileManager::Get().Delete(*PidFilePath);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	uint32 GrandchildPid = static_cast<uint32>(FCString::Atoi(*GrandchildPidStr));
	if (GrandchildPid == 0)
	{
		UE_LOGF(LogTestPAL, Error, "Grandchild PID file contained invalid data: '%ls'.", *GrandchildPidStr);
		FPlatformProcess::TerminateProc(ChildHandle, true);
		FPlatformProcess::WaitForProc(ChildHandle);
		FPlatformProcess::CloseProc(ChildHandle);
		IFileManager::Get().Delete(*PidFilePath);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}
	UE_LOGF(LogTestPAL, Display, "Grandchild PID is %u. Verifying both processes are running before the kill.",
		GrandchildPid);

	// Sanity-check: both must be alive before we test the kill
	if (!FPlatformProcess::IsProcRunning(ChildHandle))
	{
		UE_LOGF(LogTestPAL, Error, "Child (pid %u) is not running before TerminateProc - test invalid.", ChildPid);
		FPlatformProcess::CloseProc(ChildHandle);
		IFileManager::Get().Delete(*PidFilePath);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	if (!FPlatformProcess::IsApplicationRunning(GrandchildPid))
	{
		UE_LOGF(LogTestPAL, Error, "Grandchild (pid %u) is not running before TerminateProc - test invalid.", GrandchildPid);
		FPlatformProcess::TerminateProc(ChildHandle, true);
		FPlatformProcess::WaitForProc(ChildHandle);
		FPlatformProcess::CloseProc(ChildHandle);
		IFileManager::Get().Delete(*PidFilePath);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "Child (%u) and grandchild (%u) both confirmed running. Calling TerminateProc with KillTree=true.",
		ChildPid, GrandchildPid);

	FPlatformProcess::TerminateProc(ChildHandle, true /* KillTree */);

	// Poll for the child to die (it exits gracefully on SIGTERM)
	const double KillTimeoutSeconds = 10.0;
	StartTime = FPlatformTime::Seconds();
	while (FPlatformProcess::IsProcRunning(ChildHandle) && FPlatformTime::Seconds() - StartTime < KillTimeoutSeconds)
	{
		FPlatformProcess::Sleep(0.1f);
	}
	bool bChildDead = !FPlatformProcess::IsProcRunning(ChildHandle);

	// Poll for the grandchild to die and be reaped (adopted by init after child exits)
	StartTime = FPlatformTime::Seconds();
	bool bGrandchildDead = false;
	while (FPlatformTime::Seconds() - StartTime < KillTimeoutSeconds)
	{
		if (!FPlatformProcess::IsApplicationRunning(GrandchildPid))
		{
			bGrandchildDead = true;
			break;
		}
		FPlatformProcess::Sleep(0.1f);
	}

	if (bChildDead)
	{
		FPlatformProcess::WaitForProc(ChildHandle);
	}

	// Best-effort cleanup of any surviving processes so CI doesn't leak them.
	if (!bChildDead)
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: child (pid %u) is still running after TerminateProc KillTree=true.", ChildPid);
		FPlatformProcess::TerminateProc(ChildHandle, true);
		FPlatformProcess::WaitForProc(ChildHandle);
	}
	if (!bGrandchildDead)
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: grandchild (pid %u) is still running after TerminateProc KillTree=true.", GrandchildPid);
		FProcHandle GrandchildHandle = FPlatformProcess::OpenProcess(GrandchildPid);
		if (GrandchildHandle.IsValid())
		{
			FPlatformProcess::TerminateProc(GrandchildHandle, false);
			FPlatformProcess::WaitForProc(GrandchildHandle);
			FPlatformProcess::CloseProc(GrandchildHandle);
		}
	}

	FPlatformProcess::CloseProc(ChildHandle);
	IFileManager::Get().Delete(*PidFilePath);

	if (bChildDead && bGrandchildDead)
	{
		UE_LOGF(LogTestPAL, Display, "SUCCESS: child (pid %u) and grandchild (pid %u) were both terminated by TerminateProc KillTree=true.",
			ChildPid, GrandchildPid);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return (bChildDead && bGrandchildDead) ? 0 : 1;
}

/**
 * OpenProcess lifecycle test.
 * Spawns a child, obtains an OpenProcess handle, then exercises
 * IsProcRunning, TerminateProc (KillTree), WaitForProc, and CloseProc
 * through the OpenProcess code path.
 */
int32 ProcOpenProcessTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	UE_LOGF(LogTestPAL, Display, "Running proc-openprocess test (pid %d).",
		FPlatformProcess::GetCurrentProcessId());

	// Spawn a child that spins until killed (reuse the grandchild spinner role)
	FString WorkerName = FPlatformProcess::ExecutablePath();
	uint32 ChildPid = 0;
	FProcHandle SpawnHandle = FPlatformProcess::CreateProc(*WorkerName, TEXT(ARG_PROC_KILLTREE_TEST_GRANDCHILD),
		false, false, false, &ChildPid, -1, nullptr, nullptr);

	if (!SpawnHandle.IsValid())
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: could not spawn child process.");
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "Spawned child (pid %u). Waiting briefly for it to start.", ChildPid);
	FPlatformProcess::Sleep(0.5f);

	// Open a second handle to the same process via OpenProcess (the code path under test)
	FProcHandle OpenHandle = FPlatformProcess::OpenProcess(ChildPid);
	if (!OpenHandle.IsValid())
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: OpenProcess(%u) returned invalid handle.", ChildPid);
		FPlatformProcess::TerminateProc(SpawnHandle, true);
		FPlatformProcess::WaitForProc(SpawnHandle);
		FPlatformProcess::CloseProc(SpawnHandle);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	// Verify IsProcRunning returns true through the OpenProcess handle
	if (!FPlatformProcess::IsProcRunning(OpenHandle))
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: IsProcRunning returned false for a live process via OpenProcess handle.");
		FPlatformProcess::TerminateProc(SpawnHandle, true);
		FPlatformProcess::WaitForProc(SpawnHandle);
		FPlatformProcess::CloseProc(SpawnHandle);
		FPlatformProcess::CloseProc(OpenHandle);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "IsProcRunning(OpenHandle) confirmed child alive. Calling TerminateProc(OpenHandle, KillTree=true).");

	// Kill the child through the OpenProcess handle with KillTree
	FPlatformProcess::TerminateProc(OpenHandle, true);

	// Wait for it through the OpenProcess handle (detects zombie state and returns)
	FPlatformProcess::WaitForProc(OpenHandle);

	// Reap the child via the SpawnHandle so it transitions from zombie to fully gone.
	// IsProcRunning uses kill(pid, 0) which returns 0 for zombies, so we must reap first.
	FPlatformProcess::WaitForProc(SpawnHandle);

	// Verify it's dead through the OpenProcess handle
	if (FPlatformProcess::IsProcRunning(OpenHandle))
	{
		UE_LOGF(LogTestPAL, Error, "FAILED: child (pid %u) still running after TerminateProc+WaitForProc via OpenProcess handle.", ChildPid);
		FPlatformProcess::CloseProc(OpenHandle);
		// SpawnHandle was already reaped via WaitForProc above; just close it.
		FPlatformProcess::CloseProc(SpawnHandle);
		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "SUCCESS: child (pid %u) terminated and confirmed dead via OpenProcess handle.", ChildPid);

	FPlatformProcess::CloseProc(OpenHandle);
	FPlatformProcess::CloseProc(SpawnHandle);

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Tests a single file.
 */
void TestCaseInsensitiveFile(const FString & Filename, const FString & WrongFilename)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	IFileHandle * CreationHandle = PlatformFile.OpenWrite(*Filename);
	checkf(CreationHandle, TEXT("Could not create a test file for '%s'"), *Filename);
	delete CreationHandle;
	
	IFileHandle* CheckGoodHandle = PlatformFile.OpenRead(*Filename);
	checkf(CheckGoodHandle, TEXT("Could not open a test file for '%s' (zero probe)"), *Filename);
	delete CheckGoodHandle;
	
	IFileHandle* CheckWrongCaseRelHandle = PlatformFile.OpenRead(*WrongFilename);
	checkf(CheckWrongCaseRelHandle, TEXT("Could not open a test file for '%s'"), *WrongFilename);
	delete CheckWrongCaseRelHandle;
	
	PlatformFile.DeleteFile(*Filename);
}

/**
 * Case-(in)sensitivity test/
 */
int32 CaseTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();
	
	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running case sensitivity test.");
	
	TestCaseInsensitiveFile(TEXT("Test.Test"), TEXT("teSt.teSt"));
	
	FString File(TEXT("Test^%!CaseInsens"));
	FString AbsFile = FPaths::ConvertRelativePathToFull(File);
	FString AbsFileUpper = AbsFile.ToUpper();

	TestCaseInsensitiveFile(AbsFile, AbsFileUpper);
	
	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Message box test/
 */
int32 MessageBoxTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running message box test.");
	
	FString Display(TEXT("I am a big big string in a big big game, it's not a big big thing if you print me. But I do do feel that I do do will be displayed wrong, displayed wrong...  or not."));
	FString Caption(TEXT("I am a big big caption in a big big game, it's not a big big thing if you print me. But I do do feel that I do do will be displayed wrong, displayed wrong... or not."));
	EAppReturnType::Type Result = FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, *Display, *Caption);

	UE_LOGF(LogTestPAL, Display, "MessageBoxExt result: %d.", static_cast<int32>(Result));
	
	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * ************  Thread singleton test *****************
 */

/**
 * Per-thread singleton
 */
struct FPerThreadTestSingleton : public TThreadSingleton<FPerThreadTestSingleton>
{
	FPerThreadTestSingleton()
	{
		UE_LOGF(LogTestPAL, Log, "FPerThreadTestSingleton (this=%p) created for thread %d",
			this,
			FPlatformTLS::GetCurrentThreadId());
	}

	void DoSomething()
	{
		UE_LOGF(LogTestPAL, Log, "Thread %d is about to quit", FPlatformTLS::GetCurrentThreadId());
	}

	virtual ~FPerThreadTestSingleton()
	{
		UE_LOGF(LogTestPAL, Log, "FPerThreadTestSingleton (%p) destroyed for thread %d",
			this,
			FPlatformTLS::GetCurrentThreadId());
	}
};

//DECLARE_THREAD_SINGLETON( FPerThreadTestSingleton );

/**
 * Thread runnable
 */
struct FSingletonTestingThread : public FRunnable
{
	virtual uint32 Run()
	{
		FPerThreadTestSingleton& Dummy = FPerThreadTestSingleton::Get();

		FPlatformProcess::Sleep(3.0f);

		Dummy.DoSomething();
		return 0;
	}
};

/**
 * Thread singleton test
 */
int32 ThreadSingletonTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running thread singleton test.");

	const int kNumTestThreads = 10;

	FSingletonTestingThread * RunnableArray[kNumTestThreads] = { nullptr };
	FRunnableThread * ThreadArray[kNumTestThreads] = { nullptr };

	// start all threads
	for (int Idx = 0; Idx < kNumTestThreads; ++Idx)
	{
		RunnableArray[Idx] = new FSingletonTestingThread();
		ThreadArray[Idx] = FRunnableThread::Create(RunnableArray[Idx],
			*FString::Printf(TEXT("TestThread%d"), Idx));
	}

	GLog->FlushThreadedLogs();
	GLog->Flush();

	// join all threads
	for (int Idx = 0; Idx < kNumTestThreads; ++Idx)
	{
		ThreadArray[Idx]->WaitForCompletion();
		delete ThreadArray[Idx];
		ThreadArray[Idx] = nullptr;
		delete RunnableArray[Idx];
		RunnableArray[Idx] = nullptr;
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

#if PLATFORM_LINUX

static float ToMB(uint64 Value)
{
	return Value * (1.0 / (1024.0 * 1024.0));
}

#endif // PLATFORM_LINUX

/**
 * Sysinfo test
 */
int32 SysInfoTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running system info test.");

	bool bIsRunningOnBattery = FPlatformMisc::IsRunningOnBattery();
	UE_LOGF(LogTestPAL, Display, "  FPlatformMisc::IsRunningOnBattery() = %ls", bIsRunningOnBattery ? TEXT("true") : TEXT("false"));

	GenericApplication * PlatformApplication = FPlatformApplicationMisc::CreateApplication();
	checkf(PlatformApplication, TEXT("Could not create platform application!"));
	bool bIsMouseAttached = PlatformApplication->IsMouseAttached();
	UE_LOGF(LogTestPAL, Display, "  FPlatformMisc::IsMouseAttached() = %ls", bIsMouseAttached ? TEXT("true") : TEXT("false"));

	FString OSInstanceGuid = FPlatformMisc::GetOperatingSystemId();
	UE_LOGF(LogTestPAL, Display, "  FPlatformMisc::GetOperatingSystemId() = '%ls'", *OSInstanceGuid);

	FString UserDir = FPlatformProcess::UserDir();
	UE_LOGF(LogTestPAL, Display, "  FPlatformProcess::UserDir() = '%ls'", *UserDir);

	FString UserHomeDir = FPlatformProcess::UserHomeDir();
	UE_LOGF(LogTestPAL, Display, "  FPlatformProcess::UserHomeDir() = '%ls'", *UserHomeDir);

	FString UserTempDir = FPlatformProcess::UserTempDir();
	UE_LOGF(LogTestPAL, Display, "  FPlatformProcess::UserTempDir() = '%ls'", *UserTempDir);

	FString ApplicationSettingsDir = FPlatformProcess::ApplicationSettingsDir();
	UE_LOGF(LogTestPAL, Display, "  FPlatformProcess::ApplicationSettingsDir() = '%ls'", *ApplicationSettingsDir);

	FString ApplicationCurrentWorkingDir = FPlatformProcess::GetCurrentWorkingDirectory();
	UE_LOGF(LogTestPAL, Display, "  FPlatformProcess::GetCurrentWorkingDirectory() = '%ls'", *ApplicationCurrentWorkingDir);

	FPlatformMemory::DumpStats(*GLog);

#if PLATFORM_LINUX
	FExtendedPlatformMemoryStats StatsEx = FUnixPlatformMemory::GetExtendedStats();

	UE_LOGF(LogTestPAL, Display, "Shared_Clean:%.2f MB, Shared_Dirty:%.2f MB",
		ToMB(StatsEx.Shared_Clean), ToMB(StatsEx.Shared_Dirty));
	UE_LOGF(LogTestPAL, Display, "Private_Clean:%.2fMB Private_Dirty:%.2fMB",
		ToMB(StatsEx.Private_Clean), ToMB(StatsEx.Private_Dirty));

	if (StatsEx.bIs_KSM_Enabled)
	{
		UE_LOGF(LogTestPAL, Display, "KSM_Process_Profit:%.2fMB",
			ToMB(StatsEx.KSM_Process_Profit));
	}
#endif // PLATFORM_LINUX

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Crash test
 */
int32 CrashTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running crash test (this should not exit).");

	UE_LOG_CONTEXT("FunctionName", "CrashTest");

	if (FParse::Param(CommandLine, TEXT("logfatal")))
	{
		UE_LOGF(LogTestPAL, Fatal, "  LogFatal!");
	}
	else if (FParse::Param(CommandLine, TEXT("check")))
	{
		checkf(false, TEXT("  checkf!"));
	}
	else if (FParse::Param(CommandLine, TEXT("ensure")))
	{
		ensureMsgf(false, TEXT("  ensureMsgf!"));
	}
	else if (FParse::Param(CommandLine, TEXT("unaligned")))
	{
		FQuat Quat[2];
		uint8* BytePtr = (uint8*) &Quat;
		FQuat* QuatBadPtr = (FQuat*)(BytePtr + 3);
		*QuatBadPtr = FQuat::Identity;

		// This never going to be called, but kept here so entire block is not optimized out
		UE_LOGF(LogTestPAL, Warning, "Quat.X = %f", Quat[0].X);
	}
	else
	{
		*(int *)0x10 = 0x11;
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Ensure test
 */
int32 EnsureTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running ensure test.");

	// try ensures first (each ensure fires just once)
	{
		UE_LOGF(LogTestPAL, Display, "Trying 5 ensureAlways 5 times.");
		UE_LOGF(LogTestPAL, Display, "-------------------------------------------------------------------------");
		for (int IdxEnsure = 0; IdxEnsure < 5; ++IdxEnsure)
		{
			{
				UE_LOGF(LogTestPAL, Display, "*************** FIRST ensureAlways #%d time ***************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FIRST ensureAlways() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensureAlways(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "*************** SECOND ensureAlways #%d time ***************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled SECOND ensureAlways() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensureAlways(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "*************** THIRD ensureAlways #%d time ***************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled THIRD ensureAlways() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensureAlways(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "*************** FOURTH ensureAlways #%d time ***************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FOURTH ensureAlways() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensureAlways(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "*************** FIFTH ensureAlways #%d time ***************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FIFTH ensureAlways() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensureAlways(false);
			}
		}

		UE_LOGF(LogTestPAL, Display, "Trying 10 ensures 5 times.");
		UE_LOGF(LogTestPAL, Display, "-------------------------------------------------------------------------");
		for (int IdxEnsure = 0; IdxEnsure < 5; ++IdxEnsure)
		{
			{
				UE_LOGF(LogTestPAL, Display, "****************** FIRST ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FIRST ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** SECOND ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled SECOND ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** THIRD ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled THIRD ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** FOURTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FOURTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** FIFTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled FIFTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** SIXTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled SIXTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** SEVENTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled SEVENTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** EIGTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled EIGHTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** NINETH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled NINETH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

			{
				UE_LOGF(LogTestPAL, Display, "****************** TENTH ensure #%d time ******************", IdxEnsure);
				FScopeLogTime EnsureLogTime(*FString::Printf(TEXT("Handled TENTH ensure() #%d time"), IdxEnsure), nullptr, FScopeLogTime::ScopeLog_Seconds);
				ensure(false);
			}

		}

	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * String Precision test
 */
int32 StringPrecisionTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running string precision test.");

	const FString TestString(TEXT("TestString"));
	int32 Indent = 15;
	UE_LOGF(LogTestPAL, Display, "%*ls", Indent, *TestString);
	UE_LOGF(LogTestPAL, Display, "Begining of the line %*ls", Indent, *TestString);
	UE_LOGF(LogTestPAL, Display, "%*ls end of the line", Indent, *TestString);

	int Width = 2;
	for (uint32 Idx = 0; Idx < 10; ++Idx)
	{
		UE_LOGF(LogTestPAL, Display, "DynSize: %d SignedDynFormat%0*d", Width, Width, Idx);
		UE_LOGF(LogTestPAL, Display, "DynSize: %d Size_tDynFormat%0*zd", Width, Width, static_cast<size_t>(Idx));
		++Width;
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Test Push/PopDll
 */
int32 DynamicLibraryTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Attempting to load Steam library");

	FString RootSteamPath;
	FString LibraryName;

	if (PLATFORM_LINUX)
	{
		RootSteamPath = FPaths::EngineDir() / FString(TEXT("Binaries/ThirdParty/Steamworks/*"));

		IFileManager& PlatformFileManager = IFileManager::Get();
		TArray<FString> FoundDirectories;
		PlatformFileManager.FindFiles(FoundDirectories, *RootSteamPath, false, true);

		// Just use the first directory we find, this test does not have to be very sophisticated.
		if (FoundDirectories.Num() > 0)
		{
			// This only gives us directories, so remove the wildcard from our initial search
			RootSteamPath.RemoveFromEnd(TEXT("*"));
			// And append the directory name we found.
			RootSteamPath += FoundDirectories[0];
		}
		else
		{
			UE_LOGF(LogTestPAL, Fatal, "Could not find any steam versions.");
		}

		RootSteamPath += TEXT("/x86_64-unknown-linux-gnu/");
		LibraryName = TEXT("libsteam_api.so");
	}
	else
	{
		UE_LOGF(LogTestPAL, Fatal, "This test is not implemented for this platform.");
	}

	FPlatformProcess::PushDllDirectory(*RootSteamPath);
	void* SteamDLLHandle = FPlatformProcess::GetDllHandle(*LibraryName);
	FPlatformProcess::PopDllDirectory(*RootSteamPath);

	if (SteamDLLHandle == nullptr)
	{
		// try bundled one
		UE_LOGF(LogTestPAL, Error, "Could not load via Push/PopDll, loading directly.");
		SteamDLLHandle = FPlatformProcess::GetDllHandle(*(RootSteamPath + LibraryName));

		if (SteamDLLHandle == nullptr)
		{
			UE_LOGF(LogTestPAL, Fatal, "Could not load Steam library!");
		}
	}

	if (SteamDLLHandle)
	{
		UE_LOGF(LogTestPAL, Log, "Loaded Steam library at %p", SteamDLLHandle);
		FPlatformProcess::FreeDllHandle(SteamDLLHandle);
		SteamDLLHandle = nullptr;
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}



/**
 * FMalloc::GetAllocationSize() test
 */
int32 GetAllocationSizeTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running GMalloc->GetAllocationSize() test.");

	struct Allocation
	{
		void *		Memory;
		SIZE_T		RequestedSize;
		SIZE_T		Alignment;
		SIZE_T		ActualSize;
	};

	TArray<Allocation> Allocs;
	SIZE_T TotalMemoryRequested = 0, TotalMemoryAllocated = 0;

	// force proxy malloc
	FMalloc* OldGMalloc = UE::Private::GMalloc;
	UE::Private::GMalloc = new FMallocPoisonProxy(UE::Private::GMalloc);

	// allocate all the memory and initialize with 0
	for (uint32 Size = 16; Size < 4096; Size += 16)
	{
		for (SIZE_T AlignmentPower = 4; AlignmentPower <= 7; ++AlignmentPower)
		{
			SIZE_T Alignment = ((SIZE_T)1 << AlignmentPower);

			Allocation New;
			New.RequestedSize = Size;
			New.Alignment = Alignment;
			New.Memory = UE::Private::GMalloc->Malloc(New.RequestedSize, New.Alignment);
			if (!UE::Private::GMalloc->GetAllocationSize(New.Memory, New.ActualSize))
			{
				UE_LOGF(LogTestPAL, Fatal, "Could not get allocation size for %p", New.Memory);
			}
			FMemory::Memzero(New.Memory, New.RequestedSize);

			TotalMemoryRequested += New.RequestedSize;
			TotalMemoryAllocated += New.ActualSize;

			Allocs.Add(New);
		}
	}

	UE_LOGF(LogTestPAL, Log, "Allocated %llu memory (%llu requested) in %d chunks", TotalMemoryAllocated, TotalMemoryRequested, Allocs.Num());

	if (FParse::Param(CommandLine, TEXT("realloc")))
	{
		for (Allocation & Alloc : Allocs)
		{
			// resize
			Alloc.RequestedSize += 16;
			Alloc.Memory = UE::Private::GMalloc->Realloc(Alloc.Memory, Alloc.RequestedSize, Alloc.Alignment);
			FMemory::Memzero(Alloc.Memory, Alloc.RequestedSize);
		}
	}
	else
	{
		for (Allocation & Alloc : Allocs)
		{
			// only fill the difference, if any
			if (Alloc.ActualSize > Alloc.RequestedSize)
			{
				SIZE_T Difference = Alloc.ActualSize - Alloc.RequestedSize;
				FMemory::Memset((void *)((SIZE_T)Alloc.Memory + Alloc.RequestedSize), 0xAA, Difference);
			}
		}
	}

	// check if any alloc got stomped
	for (Allocation & Alloc : Allocs)
	{
		for (SIZE_T Idx = 0; Idx < Alloc.RequestedSize; ++Idx)
		{
			if (((const uint8 *)Alloc.Memory)[Idx] != 0)
			{
				UE_LOGF(LogTestPAL, Fatal, "Allocation at %p (offset %llu) got stomped with 0x%x",
					Alloc.Memory, Idx, ((const uint8 *)Alloc.Memory)[Idx]
					);
			}
		}
	}

	UE_LOGF(LogTestPAL, Log, "No memory stomping detected");

	for (Allocation & Alloc : Allocs)
	{
		UE::Private::GMalloc->Free(Alloc.Memory);
	}
	UE::Private::GMalloc = OldGMalloc;

	Allocs.Empty();

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Strings allocation test
 */
int32 StringsAllocationTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running StringsAllocationTest test.");

	const int32 NumOfStrings = 1000000;
	const TCHAR* SampleText = TEXT("Lorem ipsum dolor sit amet");

	FString** Strings = new FString*[NumOfStrings];

	UE_LOGF(LogTestPAL, Display, "Allocating %u strings '%ls'", NumOfStrings, SampleText);

	for(int32 i = 0; i < NumOfStrings; ++i)
	{
		Strings[i] = new FString(SampleText);
	}

	if (GWarn)
	{
		UE::Private::GMalloc->DumpAllocatorStats(*GWarn);
	}

	for(int32 i = 0; i < NumOfStrings; ++i)
	{
		delete Strings[i];
	}
	delete [] Strings;

	// UE::Private::GMalloc = OldGMalloc;
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Thread runnable
 * bUseGenericMisc Whether to use FGenericPlatformMisc CreateGuid
 */
template<bool bUseGenericMisc>
struct FCreateGuidThread : public FRunnable
{
	/** Number of CreateGuid calls to make. */
	int32	NumCalls;

	FCreateGuidThread(int32 InNumCalls)
		:	FRunnable()
		,	NumCalls(InNumCalls)
	{
	}

	virtual uint32 Run()
	{
		FGuid Result;

		for (int32 IdxRun = 0; IdxRun < NumCalls; ++IdxRun)
		{
			if (bUseGenericMisc)
			{
				FGenericPlatformMisc::CreateGuid(Result);
			}
			else
			{
				FPlatformMisc::CreateGuid(Result);
			}
		}

		return 0;
	}
};


/**
 * CreateGuid test
 */
int32 CreateGuidTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running CreateGuid test.");

	TArray<FRunnable*> RunnableArray;
	TArray<FRunnableThread*> ThreadArray;

	UE_LOGF(LogTestPAL, Display, "Accepted options:");
	UE_LOGF(LogTestPAL, Display, "    -numthreads=N");
	UE_LOGF(LogTestPAL, Display, "    -numcalls=N (how many times each thread will call CreateGuid)");
	UE_LOGF(LogTestPAL, Display, "    -norandomguids (disable SYS_getrandom syscall)");

	int32 NumTestThreads = 4, NumCalls = 1000000;
	if (FParse::Value(CommandLine, TEXT("numthreads="), NumTestThreads))
	{
		NumTestThreads = FMath::Max(1, NumTestThreads);
	}
	if (FParse::Value(CommandLine, TEXT("numcalls="), NumCalls))
	{
		NumCalls = FMath::Max(1, NumCalls);
	}

	// start all threads
	for (int Idx = 0; Idx < 2; ++Idx )
	{
		bool bUseGenericMisc = (Idx == 0);
		double WallTimeDuration = FPlatformTime::Seconds();

		for (int IdxThread = 0; IdxThread < NumTestThreads; ++IdxThread)
		{
			RunnableArray.Add(bUseGenericMisc ?
				static_cast<FRunnable*>(new FCreateGuidThread<true>(NumCalls)) :
				static_cast<FRunnable*>(new FCreateGuidThread<false>(NumCalls)) );

			ThreadArray.Add( FRunnableThread::Create(RunnableArray[IdxThread],
				*FString::Printf(TEXT("GuidTest%d"), IdxThread)) );
		}

		GLog->FlushThreadedLogs();
		GLog->Flush();

		// join all threads
		for (int IdxThread = 0; IdxThread < NumTestThreads; ++IdxThread)
		{
			ThreadArray[IdxThread]->WaitForCompletion();

			delete ThreadArray[IdxThread];
			ThreadArray[IdxThread] = nullptr;

			delete RunnableArray[IdxThread];
			RunnableArray[IdxThread] = nullptr;
		}

		WallTimeDuration = FPlatformTime::Seconds() - WallTimeDuration;

		UE_LOGF(LogTestPAL, Display, "--- Results for %ls ---",
			bUseGenericMisc ? TEXT("FGenericPlatformMisc::CreateGuid") : TEXT("FPlatformMisc::CreateGuid"));
		UE_LOGF(LogTestPAL, Display, "Total wall time: %f seconds, Threads: %d, Calls: %d",
				WallTimeDuration, NumTestThreads, NumCalls);

		ThreadArray.Empty();
		RunnableArray.Empty();
	}

	// Spew out a small sample of GUIDs to visually check we haven't horribly broken something
	UE_LOGF(LogTestPAL, Display, "--- CreateGuid Samples ---");

	for (int Idx = 0; Idx < 20; ++Idx)
	{
		FGuid GuidPlatform;
		FGuid GuidGeneric;

		FPlatformMisc::CreateGuid(GuidPlatform);
		FGenericPlatformMisc::CreateGuid(GuidGeneric);

		UE_LOGF(LogTestPAL, Display, "%3d GuidGeneric:%ls GuidPlatform:%ls", Idx,
			*GuidGeneric.ToString(EGuidFormats::DigitsWithHyphensInBraces),
			*GuidPlatform.ToString(EGuidFormats::DigitsWithHyphensInBraces));
	}

	// UE::Private::GMalloc = OldGMalloc;
	FEngineLoop::AppExit();
	return 0;
}


/** An ugly way to pass a parameters to FRunnable; shouldn't matter for this test code. */
int32 GMallocTestNumRuns = 500;
int32 GMallocTestMemoryPerThreadKB = 64*1024;	// 64 MB
bool GMallocTestTouchTMemory = false;

FThreadSafeCounter64 GTotalAllocsDone;
FThreadSafeCounter64 GTotalMemoryAllocated;

/**
 * Thread runnable
 * bUseSystemMalloc Whether to use system malloc for speed comparison.
 */
template<bool bUseSystemMalloc>
struct FMemoryAllocatingThread : public FRunnable
{
	virtual uint32 Run()
	{
		void* Ptrs[8192];

		for (int32 IdxRun = 0; IdxRun < GMallocTestNumRuns; ++IdxRun)
		{
			FMemory::Memzero(Ptrs);

			int32 NumAllocs = 0;
			uint64 MemAllocatedThisRun = 0;

			while(MemAllocatedThisRun < GMallocTestMemoryPerThreadKB * 1024 && NumAllocs < UE_ARRAY_COUNT(Ptrs))
			{
				// increase the probability of the smallest chunks, but allow fairly large (up to 16 MB)
				double Uniform = FMath::FRand();
				double Asymptote = (Uniform >= DBL_EPSILON) ? (1 / Uniform) : 1.0;
				double SkewedTowardsSmall = FMath::Min(1.0, Asymptote / 4096.0);	// 4096 is an arbitrary constant

				const int32 ChunkSize = 1 + static_cast<int32>(16384.0 * 1024.0 * SkewedTowardsSmall);
				//printf("Uniform: %f, Asymptote: %f, SkewedTowardsSmall: %f, ChunkSize: %d bytes\n", Uniform, Asymptote, SkewedTowardsSmall, ChunkSize);
				//fflush(stdout);

				void* Ptr = nullptr;
				if (bUseSystemMalloc)
				{
					Ptr = malloc(ChunkSize);
				}
				else
				{
					Ptr = FMemory::Malloc(ChunkSize);
				}

				if (LIKELY(Ptr))
				{
					Ptrs[NumAllocs++] = Ptr;
					MemAllocatedThisRun += ChunkSize;

					// touch the memory if not measuring the speed
					if (UNLIKELY(GMallocTestTouchTMemory))
					{
						FMemory::Memset(Ptr, 0xff, ChunkSize);
					}
				}
				else
				{
					break;
				}
			}

			//UE_LOGF(LogTestPAL, Display, "NumAllocs = %d, MemAllocatedThisRun=%llu", NumAllocs, MemAllocatedThisRun);
			GTotalAllocsDone.Add(NumAllocs);
			GTotalMemoryAllocated.Add(MemAllocatedThisRun);

			// allocate in somewhat atypical order to make it harder for malloc
			// (reinterpret Ptrs table as a 2D array of kAllocFieldWidth x NumAllocH allocs)
			const int32 kAllocFieldWidth = 4;
			int32 NumAllocsH = (NumAllocs / kAllocFieldWidth) + 1;
			for (int32 IdxAllocX = kAllocFieldWidth - 1; IdxAllocX >= 0; --IdxAllocX)
			{
				for (int32 IdxAllocY = 0; IdxAllocY < NumAllocsH; ++IdxAllocY)
				{
					int32 IdxAlloc = IdxAllocY * kAllocFieldWidth + IdxAllocX;
					if (LIKELY(IdxAlloc < NumAllocs))
					{
						void* Ptr = Ptrs[IdxAlloc];
						if (LIKELY(Ptr))
						{
							if (bUseSystemMalloc)
							{
								free(Ptr);
							}
							else
							{
								FMemory::Free(Ptr);
							}
						}
					}
				}
			}
		}
		return 0;
	}
};

/**
 * Malloc threading test
 */
int32 MallocThreadingTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	TArray<FRunnable*> RunnableArray;
	TArray<FRunnableThread*> ThreadArray;

	UE_LOGF(LogTestPAL, Display, "Accepted options:");
	UE_LOGF(LogTestPAL, Display, "    -systemmalloc to use system malloc");
	UE_LOGF(LogTestPAL, Display, "    -numthreads=N");
	UE_LOGF(LogTestPAL, Display, "    -memperthread=N (how much memory each thread will allocate in KB)");
	UE_LOGF(LogTestPAL, Display, "    -numruns=N (how many times the thread will allocate and free that memory)");
	UE_LOGF(LogTestPAL, Display, "    -touchmem (touch the memory to make sure it's backed by physical RAM - this can dominate the execution time)");

	bool bUseSystemMalloc = FParse::Param(CommandLine, TEXT("systemmalloc"));
	int32 NumTestThreads = 4, InNumTestThreads = 0, InNumRuns = 0, InMemPerThreadKB = 0;
	if (FParse::Value(CommandLine, TEXT("numthreads="), InNumTestThreads))
	{
		NumTestThreads = FMath::Max(1, InNumTestThreads);
	}
	if (FParse::Value(CommandLine, TEXT("numruns="), InNumRuns))
	{
		GMallocTestNumRuns = FMath::Max(1, InNumRuns);
	}
	if (FParse::Value(CommandLine, TEXT("memperthread="), InMemPerThreadKB))
	{
		GMallocTestMemoryPerThreadKB = FMath::Max(0, InMemPerThreadKB);
	}
	if (FParse::Param(CommandLine, TEXT("touchmem")))
	{
		GMallocTestTouchTMemory = true;
	}

	UE_LOGF(LogTestPAL, Display, "Running malloc threading test using %ls malloc and %d threads, each allocating %lsup to %d KB (%d MB) memory %d times.",
		bUseSystemMalloc ? TEXT("libc") : UE::Private::GMalloc->GetDescriptiveName(),
		NumTestThreads,
		GMallocTestTouchTMemory ? TEXT("and touching ") : TEXT(""),
		GMallocTestMemoryPerThreadKB, GMallocTestMemoryPerThreadKB / 1024,
		GMallocTestNumRuns
		);

	// start all threads
	double WallTimeDuration = 0.0;
	{
		FSimpleScopeSecondsCounter Duration(WallTimeDuration);
		for (int Idx = 0; Idx < NumTestThreads; ++Idx)
		{
			RunnableArray.Add( bUseSystemMalloc ? static_cast<FRunnable*>(new FMemoryAllocatingThread<true>()) : static_cast<FRunnable*>(new FMemoryAllocatingThread<false>()) );
			ThreadArray.Add( FRunnableThread::Create(RunnableArray[Idx],
				*FString::Printf(TEXT("MallocTest%d"), Idx)) );
		}

		GLog->FlushThreadedLogs();
		GLog->Flush();

		// join all threads
		for (int Idx = 0; Idx < NumTestThreads; ++Idx)
		{
			ThreadArray[Idx]->WaitForCompletion();
			delete ThreadArray[Idx];
			ThreadArray[Idx] = nullptr;
			delete RunnableArray[Idx];
			RunnableArray[Idx] = nullptr;
		}
	}
	UE_LOGF(LogTestPAL, Display, "--- Results for %ls malloc ---", bUseSystemMalloc ? TEXT("libc") : UE::Private::GMalloc->GetDescriptiveName());
	UE_LOGF(LogTestPAL, Display, "Total wall time:        %f seconds", WallTimeDuration);
	UE_LOGF(LogTestPAL, Display, "Total allocations done: %llu", GTotalAllocsDone.GetValue());
	UE_LOGF(LogTestPAL, Display, "Total memory allocated: %llu bytes (%llu KB, %llu MB)", GTotalMemoryAllocated.GetValue(),
		GTotalMemoryAllocated.GetValue() / 1024, GTotalMemoryAllocated.GetValue() / (1024 * 1024));

	double AllocsPerSecond = (WallTimeDuration > 0.0) ? GTotalAllocsDone.GetValue() / WallTimeDuration : 0.0;
	double BytesPerSecond = (WallTimeDuration > 0.0) ? GTotalMemoryAllocated.GetValue() / WallTimeDuration : 0.0;

	UE_LOGF(LogTestPAL, Display, "Speed in allocs:        %.1f Kallocs/sec (%.1f allocs/sec)%ls", AllocsPerSecond / 1000.0, AllocsPerSecond,
		GMallocTestTouchTMemory ? TEXT("- less relevant since memory was touched") : TEXT(""));
	UE_LOGF(LogTestPAL, Display, "Speed in bytes:         %.1f MB/sec (%.1f KB/sec, %.1f bytes/sec)%ls", BytesPerSecond / (1024.0 * 1024.0), BytesPerSecond / 1024.0, BytesPerSecond,
		GMallocTestTouchTMemory ? TEXT("- less relevant since memory is touched") : TEXT(""));

	FPlatformMemoryStats Stats = FPlatformMemory::GetStats();
	UE_LOGF(LogTestPAL, Display, "Peak used resident RAM: %llu MB (%llu KB, %llu bytes)", Stats.PeakUsedPhysical / (1024 * 1024), Stats.PeakUsedPhysical / 1024, Stats.PeakUsedPhysical);
	UE_LOGF(LogTestPAL, Display, "Peak used virtual RAM:  %llu MB (%llu KB, %llu bytes)", Stats.PeakUsedVirtual / (1024 * 1024), Stats.PeakUsedVirtual / 1024, Stats.PeakUsedVirtual);

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

/**
 * Replays a malloc save file, streaming it from the disk. Waits for Ctrl-C until exiting.
 * 
 * @param ReplayFileName file name to read. Must be reachable from cwd (current working directory) or absolute
 * @param OperationToStopAfter number of operation to stop on - no further reads will be done. Useful to compare peak usage in the middle of file.
 * @param bSuppressErrors Whether to print errors
 */
void ReplayMallocFile(const FString& ReplayFileName, uint64 OperationToStopAfter, bool bSuppressErrors)
{
#if PLATFORM_WINDOWS
	FILE* ReplayFile = nullptr;
	if (fopen_s(&ReplayFile, TCHAR_TO_UTF8(*ReplayFileName), "rb") != 0 || ReplayFile == nullptr)
#else
	FILE* ReplayFile = fopen(TCHAR_TO_UTF8(*ReplayFileName), "rb");
	if (ReplayFile == nullptr)
#endif // PLATFORM_WINDOWS
	{
		return;
	}

	// For file format, see FMallocReplayProxy

	// skip the first line since it contains column headers
	{
		char DummyBuffer[4096];
		fgets(DummyBuffer, sizeof(DummyBuffer), ReplayFile);
	}

	double WallTimeDuration = 0.0;

	uint64 OperationNumber = 0;
	TMap<uint64, void *> FilePointerToRamPointers;
	for(;;)
	{
		FSimpleScopeSecondsCounter Duration(WallTimeDuration);

		char OpBuffer[128] = {0};
		uint64 PtrOut, PtrIn, Size, Ordinal;
		uint32 Alignment;
#if PLATFORM_WINDOWS
		if (fscanf_s(ReplayFile, "%s %llu %llu %llu %u\t# %llu\n", OpBuffer, static_cast<unsigned int>(sizeof(OpBuffer)), &PtrOut, &PtrIn, &Size, &Alignment, &Ordinal) != 6)
#else
		if (fscanf(ReplayFile, "%s %llu %llu %llu %u\t# %llu\n", OpBuffer, &PtrOut, &PtrIn, &Size, &Alignment, &Ordinal) != 6)
#endif // PLATFORM_WINDOWS
		{
			UE_LOGF(LogTestPAL, Display, "Hit end of the replay file on %llu-th operation.", OperationNumber);
			break;
		}

		if (!FCStringAnsi::Strcmp(OpBuffer, "Malloc"))
		{
			if (FilePointerToRamPointers.Find(PtrOut) == nullptr)
			{
				void* Result = FMemory::Malloc(Size, Alignment);
				FilePointerToRamPointers.Add(PtrOut, Result);
			}
			else if (!bSuppressErrors)
			{
				UE_LOGF(LogTestPAL, Error, "Replay file contains operation # %llu that returned pointer %llu, which was already allocated at that moment. Skipping.", Ordinal, PtrOut);
			}
		}
		else if (!FCStringAnsi::Strcmp(OpBuffer, "Realloc"))
		{
			void* PtrToRealloc = nullptr;
			if (PtrIn != 0)
			{
				void** RamPointer = FilePointerToRamPointers.Find(PtrIn);
				if (RamPointer != nullptr)
				{
					PtrToRealloc = *RamPointer;
				}
				else if (!bSuppressErrors)
				{
					UE_LOGF(LogTestPAL, Error, "Replay file contains operation # %llu to reallocate pointer %llu, which was not allocated at that moment. Substituting with nullptr.", Ordinal, PtrIn);
				}
			}

			void* Result = FMemory::Realloc(PtrToRealloc, Size, Alignment);
			FilePointerToRamPointers.Remove(PtrIn);
			FilePointerToRamPointers.Add(PtrOut, Result);
		}
		else if (!FCStringAnsi::Strcmp(OpBuffer, "Free"))
		{
			void* PtrToFree = nullptr;
			if (PtrIn != 0)
			{
				void** RamPointer = FilePointerToRamPointers.Find(PtrIn);
				if (RamPointer != nullptr)
				{
					PtrToFree = *RamPointer;
				}
				else if (!bSuppressErrors)
				{
					UE_LOGF(LogTestPAL, Error, "Replay file contains operation # %llu to free pointer %llu, which was not allocated at that moment. Substituting with nullptr.", Ordinal, PtrIn);
				}
			}

			FMemory::Free(PtrToFree);
			FilePointerToRamPointers.Remove(PtrIn);
		}
		else if (!bSuppressErrors)
		{
			UE_LOGF(LogTestPAL, Error, "Replay file contains unknown operation '%ls', skipping.", ANSI_TO_TCHAR(OpBuffer));
		}

		++OperationNumber;
		if (OperationNumber >= OperationToStopAfter)
		{
			UE_LOGF(LogTestPAL, Display, "Stopping after %llu-th operation.", OperationNumber);
			break;
		}
	}

	UE_LOGF(LogTestPAL, Display, "Replayed %llu operations in %f seconds, waiting for Ctrl-C to proceed further. You can now examine heap/process state.", OperationNumber, WallTimeDuration);

	while(!IsEngineExitRequested())
	{
		FPlatformProcess::Sleep(1.0);
	}
}

/**
 * Malloc replaying test
 */
int32 MallocReplayTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	FString ReplayFileName;
	if (FParse::Value(CommandLine, TEXT("replayfile="), ReplayFileName))
	{
		uint64 OperationToStopAfter = (uint64)(-1);
		if (!FParse::Value(CommandLine, TEXT("stopafter="), OperationToStopAfter))
		{
			UE_LOGF(LogTestPAL, Display, "You can pass -stopafter=N to stop after Nth operation.");
		}
		
		bool bSuppressErrors = FParse::Param(CommandLine, TEXT("suppresserrors"));

		ReplayMallocFile(ReplayFileName, OperationToStopAfter, bSuppressErrors);
	}
	else
	{
		UE_LOGF(LogTestPAL, Error, "No file to replay. Pass -replayfile=PathToFile.txt");
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}


/**
 * Translates priorities into strings.
 * For priorities values, see GenericPlatformAffinity.h.
 */
const TCHAR* ThreadPrioToString(EThreadPriority Prio)
{
	switch(Prio)
	{
#define RET_PRIO(PrioName)	case PrioName: return TEXT(#PrioName);
		RET_PRIO(TPri_Normal)
		RET_PRIO(TPri_AboveNormal)
		RET_PRIO(TPri_BelowNormal)
		RET_PRIO(TPri_Highest)
		RET_PRIO(TPri_Lowest)
		RET_PRIO(TPri_SlightlyBelowNormal)
		RET_PRIO(TPri_TimeCritical)
#undef RET_PRIO
		default:
			checkf(false, TEXT("Unknown priority!"));
			break;
	}

	return nullptr;
}

/**
 * Thread (will be run with different prios)
 */
struct FThreadPrioTester : public FRunnable
{
	/** Priority assigned to this thread (for printing purposes). */
	EThreadPriority			Prio;

	/** Counter incremented by this thread. */
	uint64					Counter;

	/** Time to run (all threads should have the same value. */
	double					SecondsToRun;

	/** How much time the thread was actually running. Should be close to SecondsToRun, but may differ. */
	double					SecondsActuallyRan;

	FThreadPrioTester(EThreadPriority InPrio, double InSecondsToRun)
		:	FRunnable()
		,	Prio(InPrio)
		,	Counter(0)
		,	SecondsToRun(InSecondsToRun)
	{
	}

	virtual uint32 Run() override
	{
		double StartTime = FPlatformTime::Seconds();
		for(;;)
		{
			// don't check too often to avoid threads bottlenecking on access to clock
			if (Counter % 65536 == 0)
			{
				// account for an unlikely case that we will never get to run in the allotted time and check first
				double RanForSoFar = FPlatformTime::Seconds() - StartTime;
				if (RanForSoFar >= SecondsToRun)
				{
					SecondsActuallyRan = RanForSoFar;
					break;
				}
			}

			++Counter;
		}

		return 0;
	}
};

/**
 * Thread priorities test
 */
int32 ThreadPriorityTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	UE_LOGF(LogTestPAL, Display, "Accepted options:");
	UE_LOGF(LogTestPAL, Display, "    -secondstorun=N (for how long to run)");
	UE_LOGF(LogTestPAL, Display, "    -numthreadgroups=N (how many groups of threads to run - so we can saturate the CPU)");

	float SecondsToRun = 16.0f, InSecondsToRun = 0.0f;
	int32 NumThreadGroups = 8, InNumThreadGroups = 0;
	if (FParse::Value(CommandLine, TEXT("secondstorun="), InSecondsToRun))
	{
		SecondsToRun = FMath::Max(1.0f, InSecondsToRun);
	}
	if (FParse::Value(CommandLine, TEXT("numthreadgroups="), InNumThreadGroups))
	{
		NumThreadGroups = FMath::Max(1, InNumThreadGroups);
	}

	UE_LOGF(LogTestPAL, Display, "Running thread priority test (%d thread groups) for %.1f seconds.",
		NumThreadGroups,
		SecondsToRun
		);

	/*  Note that enum - as of now at least - is not ordered in any way and looks like this:
		TPri_Normal,
		TPri_AboveNormal,
		TPri_BelowNormal,
		TPri_Highest,
		TPri_Lowest,
		TPri_SlightlyBelowNormal,
		TPri_TimeCritical */

	TArray<FThreadPrioTester*> RunnableArray;
	TArray<FRunnableThread*> ThreadArray;
	for (int32 IdxGroup = 0; IdxGroup < NumThreadGroups; ++IdxGroup)
	{
		RunnableArray.Add(new FThreadPrioTester(TPri_Normal, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_AboveNormal, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_BelowNormal, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_Highest, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_Lowest, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_SlightlyBelowNormal, SecondsToRun));
		RunnableArray.Add(new FThreadPrioTester(TPri_TimeCritical, SecondsToRun));
	}

	UE_LOGF(LogTestPAL, Display, "Creating %d threads", RunnableArray.Num());
	for (int32 Idx = 0, NumThreads = RunnableArray.Num(); Idx < NumThreads; ++Idx)
	{
		ThreadArray.Add( FRunnableThread::Create(RunnableArray[Idx],
			*FString::Printf(TEXT("(%d)%s"), Idx / 7, ThreadPrioToString(RunnableArray[Idx]->Prio)), 0, RunnableArray[Idx]->Prio)
			);
	}

	// join all threads
	for (int32 Idx = 0, NumThreads = RunnableArray.Num(); Idx < NumThreads; ++Idx)
	{
		ThreadArray[Idx]->WaitForCompletion();
		delete ThreadArray[Idx];
		ThreadArray[Idx] = nullptr;
	}

	GLog->FlushThreadedLogs();
	GLog->Flush();

	UE_LOGF(LogTestPAL, Display, "--- Results ---");

	// tally up all threads (note that as of now there are seven prios)
	uint64 AllThreadsCounters[7] = {0};
	double AllThreadsTimes[7] = {0};
	uint64 TotalCount = 0;

	// describe all threads
	for (int32 Idx = 0, NumThreads = RunnableArray.Num(); Idx < NumThreads; ++Idx)
	{
		uint32 PrioIndex = static_cast<uint32>(RunnableArray[Idx]->Prio);
		if (PrioIndex >= UE_ARRAY_COUNT(AllThreadsCounters))
		{
			UE_LOGF(LogTestPAL, Fatal, "EThreadPriority enum changed and has values larger than 7 now. Revisit this code.");
		}
		else
		{
			AllThreadsCounters[PrioIndex] += RunnableArray[Idx]->Counter;
			AllThreadsTimes[PrioIndex] += RunnableArray[Idx]->SecondsActuallyRan;
			TotalCount += RunnableArray[Idx]->Counter;
		}
		delete RunnableArray[Idx];
		RunnableArray[Idx] = nullptr;
	}

	// compare to TPri_Normal priority
	uint64 BaseCount = FMath::Max(1ULL, AllThreadsCounters[static_cast<uint32>(TPri_Normal)]);

	double BaseCountDbl = static_cast<double>(BaseCount);
	uint32 OrderToPrint[7] =
	{
		static_cast<uint32>(TPri_Lowest),
		static_cast<uint32>(TPri_BelowNormal),
		static_cast<uint32>(TPri_SlightlyBelowNormal),
		static_cast<uint32>(TPri_Normal),
		static_cast<uint32>(TPri_AboveNormal),
		static_cast<uint32>(TPri_Highest),
		static_cast<uint32>(TPri_TimeCritical)
	};

	for (uint32 IdxPrio = 0; IdxPrio < UE_ARRAY_COUNT(OrderToPrint); ++IdxPrio)
	{
		uint32 Prio = OrderToPrint[IdxPrio];
		checkf(Prio < UE_ARRAY_COUNT(AllThreadsCounters), TEXT("Number of thread priority enums changed, revisit this code."));
		UE_LOGF(LogTestPAL, Display, "Threads with prio %24ls incremented counters %14llu times (%.1f x speed of TPri_Normal) during %1.f sec total",
			ThreadPrioToString(static_cast<EThreadPriority>(Prio)), AllThreadsCounters[Prio],
			static_cast<double>(AllThreadsCounters[Prio]) / BaseCountDbl,
			AllThreadsTimes[Prio]);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

// inlined/non-inlined functions for testing
namespace
{
	void FORCENOINLINE LexicalBlock()
	{
		{
			{
				ensure(false);
			}
		}
	}

	void FORCENOINLINE LabelSwitch()
	{
		switch (1)
		{
			case 1:
				ensure(false);
				break;
			default:
				break;
		}
	}

	PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
	void FORCENOINLINE LabelGoto()
	{
		goto end;
		ensure(false); // skips all of these. Just want to create a bunch of inline statements in this single non inlined one
		ensure(false);
		ensure(false);
		ensure(false);
		ensure(false);
		ensure(false);
end:
		ensure(false);
	}
	PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

	void FORCEINLINE inline_three_ensures()
	{
		ensure(false);
	}

	void FORCEINLINE inline_two_calls_inline_three()
	{
		inline_three_ensures();
	}

	void FORCEINLINE inline_one_calls_inline_two()
	{
		inline_two_calls_inline_three();
	}

	void FORCENOINLINE MultipleInlineDeep()
	{
		inline_one_calls_inline_two();
	}

	void FORCEINLINE inline_crash()
	{
		*(int*)0x1 = 0x0;
	}

	void FORCENOINLINE no_inline_to_inline_crash()
	{
		inline_crash();
	}
}

int32 InlineCallstacksTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	/*  Three unique cases that we can only test two:
	 *
	 *  LexicalBlocks (ie. scope/if/for/while blocks
	 *  Labels (ie. goto/switch statements)
	 *  Try/Catch (cannot use when execptions are disabled, but we *should* handle this case)
	 *
	 *  As well as testing, multiple deep inlining. ie. Calling multiple inline functions and still
	 *  seeing the call site
	 *
	 *  Just make sure that the functions used + their callsite into ensure(false) are correctly reported in
	 *  the callstack
	 */

	UE_LOGF(LogTestPAL, Warning, "\n*** Lexical Block ***");
	LexicalBlock();
	UE_LOGF(LogTestPAL, Warning, "\n*** Label Switch ***");
	LabelSwitch();

	UE_LOGF(LogTestPAL, Warning, "\n*** Label Goto ***");
	LabelGoto();

	UE_LOGF(LogTestPAL, Warning, "\n*** Multiple Inlined ***");
	MultipleInlineDeep();

	// Should always be the last case, as this crashes
	UE_LOGF(LogTestPAL, Warning, "\n*** Array delegate to crash***");
	TArray<TFunction<void()>> a;
	a.Push([] { no_inline_to_inline_crash(); });
	a[0]();

	return 0;
}

struct FThreadStackTest : public FRunnable
{
	FThreadStackTest(uint64 InThreadId = ~0) :
		ThreadId(InThreadId)
	{
	}

	uint64 ThreadId;

	virtual uint32 Run()
	{
		// If we dont have a valid ThreadId lets use the threads id as default
		if (ThreadId == ~0)
		{
			ThreadId = FPlatformTLS::GetCurrentThreadId();
		}

		// Hopefully this wont be to much extra stack space for a testing thread
		const SIZE_T StackTraceSize = 65536;
		ANSICHAR StackTrace[StackTraceSize] = {0};
		FPlatformStackWalk::ThreadStackWalkAndDump(StackTrace, StackTraceSize, 0, ThreadId);

		UE_LOGF(LogTestPAL, Warning, "***** ThreadStackWalkAndDump for ThreadId(%llu) ******\n%ls", ThreadId, ANSI_TO_TCHAR(StackTrace));

		const SIZE_T BackTraceSize = 100;
		uint64 BackTrace[BackTraceSize];
		int32 BackTraceCount = FPlatformStackWalk::CaptureThreadStackBackTrace(ThreadId, BackTrace, BackTraceSize);

		UE_LOGF(LogTestPAL, Warning, "***** CaptureThreadStackBackTrace for ThreadId(%llu) ******", ThreadId);
		for (int i = 0; i < BackTraceCount; i++)
		{
			UE_LOGF(LogTestPAL, Warning, "0x%llx", BackTrace[i]);
		}
		UE_LOGF(LogTestPAL, Warning, "\n\n");

		UE_LOGF(LogTestPAL, Warning, "***** ProgramCounterToHumanReadableString for BackTrace for ThreadId(%llu) ******", ThreadId);
		for (int i = 0; i < BackTraceCount; i++)
		{
			ANSICHAR TempString[1024] = {0};
			FPlatformStackWalk::ProgramCounterToHumanReadableString(i, BackTrace[i], TempString, UE_ARRAY_COUNT(TempString));

			UE_LOGF(LogTestPAL, Warning, "%ls", ANSI_TO_TCHAR(TempString));
		}

		return 0;
	}
};


int32 ThreadTraceTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	// Dump the callstack/backtrace of the thread that is running
	{
		FThreadStackTest* ThreadStackTest = new FThreadStackTest;
		FRunnableThread* Runnable = FRunnableThread::Create(ThreadStackTest, ANSI_TO_TCHAR("ThreadStackTest"), 0);

		Runnable->WaitForCompletion();
		delete ThreadStackTest;
	}

	// Dump the GGameThreadId callstack/backtrace from the thread
	{
		FThreadStackTest* ThreadStackTest = new FThreadStackTest(GGameThreadId);
		FRunnableThread* Runnable = FRunnableThread::Create(ThreadStackTest, ANSI_TO_TCHAR("ThreadStackTest"), 0);

		Runnable->WaitForCompletion();
		delete ThreadStackTest;
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}

int32 ForkTest(const TCHAR* CommandLine)
{
	// Only supported on Linux
	if (PLATFORM_LINUX)
	{
		FPlatformMisc::SetCrashHandler(NULL);
		FPlatformMisc::SetGracefulTerminationHandler();

		GEngineLoop.PreInit(CommandLine);

		if (FPlatformProcess::SupportsMultithreading())
		{
			UE_LOGF(LogTestPAL, Error, "WaitAndFork is only supported with '-nothreading' command line");
		}
		else
		{
			UE_LOGF(LogTestPAL, Warning, "About to fork. Press Ctrl+C to close parent process");

			FPlatformProcess::EWaitAndForkResult Result = FPlatformProcess::WaitAndFork();

			if (Result == FPlatformProcess::EWaitAndForkResult::Child)
			{
				UE_LOGF(LogTestPAL, Display, "Child process: %d IsChildProcess: %d", FPlatformProcess::GetCurrentProcessId(), FForkProcessHelper::IsForkedChildProcess());
			}
			else
			{
				// parent, WaitAndFork holds the parent process to spawn more children until the we are closing
				UE_LOGF(LogTestPAL, Display, "Parent process: %d IsChildProcess: %d", FPlatformProcess::GetCurrentProcessId(), FForkProcessHelper::IsForkedChildProcess());
			}
		}

		FEngineLoop::AppPreExit();
		FEngineLoop::AppExit();
	}

	return 0;
}

/** 
 * Launch the external tool as a process and wait for it to complete
 *
 * Set executable and parameters via something like:
 *
 *  ./Engine/Binaries/Linux/TestPAL exec-process exe=/bin/ls params=/tmp
 *
 */
int32 ExecProcessTest(const TCHAR *CommandLine)
{
	FString Exe;
	FString Params;

	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	if (!FParse::Value(CommandLine, TEXT("exe="), Exe))
	{
		Exe = TEXT("nonexistent-exe");
	}
	if (!FParse::Value(CommandLine, TEXT("params="), Params))
	{
		Params = TEXT("");
	}

	UE_LOGF(LogTestPAL, Display, "  Exe:'%ls' Parameters:'%ls'", *Exe, *Params);

	int32 ReturnCode;
	FString OutStdOut;
	FString OutStdErr;
	bool Ret = FPlatformProcess::ExecProcess(*Exe, *Params, &ReturnCode, &OutStdOut, &OutStdErr);

	UE_LOGF(LogTestPAL, Display, "ExecProcess returns %d", Ret);
	UE_LOGF(LogTestPAL, Display, "  ReturnCode:%d", ReturnCode);
	UE_LOGF(LogTestPAL, Display, "  StdOut:%ls", *OutStdOut);
	UE_LOGF(LogTestPAL, Display, "  StdErr:%ls", *OutStdErr);

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}

/**
 * Opens the file for reading that is rewritable by another Unreal process and keeps it so.
 *
 */
int32 FileKeepOpenTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	FString FileName;
	if (!FParse::Value(CommandLine, TEXT("-file="), FileName))
	{
		UE_LOGF(LogTestPAL, Error, "Use mandatory -file argument to point to the existing file");
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "  File to keep open:'%ls'", *FileName);

	{
		TUniquePtr<IFileHandle> File(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*FileName, true));
		if (!File)
		{
			UE_LOGF(LogTestPAL, Error, "Could not open file '%ls'", *FileName);
			return 1;
		}

		UE_LOGF(LogTestPAL, Display, "Opened the file, will keep it open until told to quit. Now use FileReplaceTest to replace it");

		UE_LOGF(LogTestPAL, Display, "IsEngineExitRequested = %d", IsEngineExitRequested());

		while(!IsEngineExitRequested())
		{
			FPlatformProcess::Sleep(1.0f);
		}

		UE_LOGF(LogTestPAL, Display, "IsEngineExitRequested = %d", IsEngineExitRequested());
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}

/**
 * Attempts to replace the file that is open for reading by another TestPAL instance.
 *
 */
int32 FileReplaceOpenTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	FString FileSourceName;
	if (!FParse::Value(CommandLine, TEXT("-filesource="), FileSourceName))
	{
		UE_LOGF(LogTestPAL, Error, "Use mandatory -filesource argument to point to the existing file");
		return 1;
	}

	FString FileTargetName;
	if (!FParse::Value(CommandLine, TEXT("-filetarget="), FileTargetName))
	{
		UE_LOGF(LogTestPAL, Error, "Use mandatory -filetarget argument to point to the existing file");
		return 1;
	}

	UE_LOGF(LogTestPAL, Display, "  File that will be replaced:'%ls'", *FileTargetName);
	UE_LOGF(LogTestPAL, Display, "  File that it will be replaced with (i.e. source):'%ls'", *FileSourceName);

	{
		FString ToFileDir = FPaths::GetPath(FileTargetName);
		FString TempDestFile = FPaths::CreateTempFilename(*ToFileDir);

		// move the target file to a temp location first, then copy source over and delete the temp
		if (!FPlatformFileManager::Get().GetPlatformFile().MoveFile(*TempDestFile, *FileTargetName) || !FPlatformFileManager::Get().GetPlatformFile().CopyFile(*FileTargetName, *FileSourceName) || 
			!FPlatformFileManager::Get().GetPlatformFile().DeleteFile(*TempDestFile))
		{
			UE_LOGF(LogTestPAL, Error, "Could not replace file '%ls'!", *FileTargetName);
			return 1;
		}

		UE_LOGF(LogTestPAL, Display, "Successfully replaced file '%ls'!", *FileTargetName);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}


/**
 * Creates a temporary file (right where the binary is located), maps it open, then attempts to delete it while keeping the handle open.
 *
 */
int32 FileDeleteMappedTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);

	FString MmappedFile = FPaths::CreateTempFilename(TEXT("."));

	UE_LOGF(LogTestPAL, Display, "  File that will be kept memory-mapped:'%ls'", *MmappedFile);
	{
		FRandomStream Rand(static_cast<int32>(FPlatformTime::Cycles()));

		auto GetRandomString = [Rand]() -> FString
			{
				int32 Size = static_cast<int32>(128 + 1024.0 * Rand.GetFraction()) * 1024;
				FString Temp;

				Temp.Reserve(Size);
				for (int32 Idx = 0; Idx < Size; ++Idx)
				{
					Temp += static_cast<TCHAR>(static_cast<uint32>(TEXT('0')) + static_cast<uint32>(36.0 * Rand.GetFraction()) * sizeof(TCHAR));
				}

				return MoveTemp(Temp);
			};

		auto SaveRandomFile = [Rand, GetRandomString](const FString& Filename) -> bool
			{
				FString Temp = GetRandomString();
				return FFileHelper::SaveStringToFile(Temp, *Filename, FFileHelper::EEncodingOptions::ForceUTF8);
			};

		if (!SaveRandomFile(MmappedFile))
		{
			UE_LOGF(LogTestPAL, Error, "Could not create file '%ls'!", *MmappedFile);
			return 1;
		}
	}

	{
		TUniquePtr<IMappedFileHandle> Handle;
		{
			FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(*MmappedFile, IPlatformFile::EOpenReadFlags::AllowDelete);
			if (Result.HasError())
			{
				UE_LOGF(LogTestPAL, Error, "Could not memory map file '%ls'!", *MmappedFile);
				return 1;
			}

			Handle = Result.StealValue();
		}

		// delete, or infinitely loop
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		UE_LOGF(LogTestPAL, Display, "Attempting to delete file '%ls', will loop forever until it suceeds to help analyze problems.", *MmappedFile);
		int64 NumTry = 1;
		// intersperse exists and delete to have a better chance for CPU profiling to find a boundary for the DeleteFile
		while (PlatformFile.FileExists(*MmappedFile) && !PlatformFile.DeleteFile(*MmappedFile))
		{
			++NumTry;
		}
		UE_LOGF(LogTestPAL, Display, "Successfully deleted file '%ls' on a %lld try!", *MmappedFile, NumTry);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return 0;
}

int32 CmdlineParseTest(const TCHAR *CommandLine)
{
	int32 NotOk = 0;	// zero when everything has passed so far
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	
	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	verify(FPlatformProcess::CreatePipe(ReadPipe, WritePipe));

	uint32 ProcessID;

	// key is the command line to test, value is the expected parse result when run through 'echo'
	TArray<TTuple<FString, FString>> tests = 
	{ 
		{ TEXT("foo"), TEXT("foo") },
		{ TEXT("\"foo bar\""), TEXT("foo bar") },
		{ TEXT("\"\"foo bar\"\""), TEXT("\"foo bar\"") },
		{ TEXT("\"foo"), TEXT("") },
		{ TEXT("\"\"foo"), TEXT("") },
		{ TEXT("-logpath=\"path with space\""), TEXT("-logpath=path with space") },
		{ TEXT("-logpath=\"\"double quoted path with space\"\""), TEXT("-logpath=\"double quoted path with space\"") },
		{ TEXT("bar \"\"foo blarg"), TEXT("bar") },
	};

	for(int i=0;i<tests.Num();i++)
	{
		FProcHandle ProcessHandle = FPlatformProcess::CreateProc(TEXT("/usr/bin/echo"), *tests[i].Key, false, false, false, &ProcessID, 0, nullptr, WritePipe);
		while (FPlatformProcess::IsProcRunning(ProcessHandle));
		FString Result = FPlatformProcess::ReadPipe(ReadPipe).TrimEnd();
		NotOk = Result.Compare(*tests[i].Value);
		if(NotOk)
		{
			UE_LOGF(LogTestPAL, Display, "CmdLine test failed: '%ls' != '%ls'\n", *Result, *tests[i].Value);
			break;
		}
	}
	
	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);

	UE_LOGF(LogTestPAL, Display, "Parse command line test %ls", (NotOk)?TEXT("failed!"):TEXT("succeeded!"));

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();

	return NotOk;
}

/**
 * Selects and runs one of test cases.
 *
 * @param ArgC Number of commandline arguments.
 * @param ArgV Commandline arguments.
 * @return Test case's return code.
 */
int32 MultiplexedMain(int32 ArgC, char* ArgV[])
{
	for (int32 IdxArg = 0; IdxArg < ArgC; ++IdxArg)
	{
		if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_TEST_CHILD))
		{
			return ProcRunAsChild(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_TEST))
		{
			return ProcRunAsParent(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_KILLTREE_TEST_GRANDCHILD))
		{
			return ProcKillTreeRunAsGrandchild(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_KILLTREE_TEST_CHILD))
		{
			return ProcKillTreeRunAsChild(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_KILLTREE_TEST))
		{
			return ProcKillTreeRunAsParent(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_PROC_OPENPROCESS_TEST))
		{
			return ProcOpenProcessTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CASE_SENSITIVITY_TEST))
		{
			return CaseTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_MESSAGEBOX_TEST))
		{
			return MessageBoxTest(*TestPAL::CommandLine);
		}
#if USE_DIRECTORY_WATCHER
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_DIRECTORY_WATCHER_TEST))
		{
			return DirectoryWatcherTest(*TestPAL::CommandLine);
		}
#endif
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_THREAD_SINGLETON_TEST))
		{
			return ThreadSingletonTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_SYSINFO_TEST))
		{
			return SysInfoTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CRASH_TEST))
		{
			return CrashTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_ENSURE_TEST))
		{
			return EnsureTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_STRINGPRECISION_TEST))
		{
			return StringPrecisionTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_DSO_TEST))
		{
			return DynamicLibraryTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_GET_ALLOCATION_SIZE_TEST))
		{
			return GetAllocationSizeTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_MALLOC_THREADING_TEST))
		{
			return MallocThreadingTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_MALLOC_REPLAY))
		{
			return MallocReplayTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_THREAD_PRIO_TEST))
		{
			return ThreadPriorityTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_INLINE_CALLSTACK_TEST))
		{
			return InlineCallstacksTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_STRINGS_ALLOCATION_TEST))
		{
			return StringsAllocationTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CREATEGUID_TEST))
		{
			return CreateGuidTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_THREADSTACK_TEST))
		{
			return ThreadTraceTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_EXEC_PROCESS_TEST))
		{
			return ExecProcessTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_FORK_TEST))
		{
			return ForkTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CMDLINE_FILE_KEEP_OPEN))
		{
			return FileKeepOpenTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CMDLINE_FILE_REPLACE_OPEN))
		{
			return FileReplaceOpenTest(*TestPAL::CommandLine);
		}
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CMDLINE_FILE_DELETE_MAPPED))
		{
			return FileDeleteMappedTest(*TestPAL::CommandLine);
		}
#if PLATFORM_LINUX
		else if (!FCStringAnsi::Strcmp(ArgV[IdxArg], ARG_CMDLINE_PARSE_TEST))
		{
			return CmdlineParseTest(*TestPAL::CommandLine);
		}
#endif
	}

	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();
	
	GEngineLoop.PreInit(*TestPAL::CommandLine);
	UE_LOGF(LogTestPAL, Warning, "Unable to find any known test name, no test started.");

	UE_LOGF(LogTestPAL, Warning, "");
	UE_LOGF(LogTestPAL, Warning, "Available test cases:");
	UE_LOGF(LogTestPAL, Warning, "  %ls: test process handling API", ANSI_TO_TCHAR( ARG_PROC_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test TerminateProc with KillTree=true (spawns a child which spawns a grandchild, then verifies both are killed)", ANSI_TO_TCHAR( ARG_PROC_KILLTREE_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test OpenProcess handle lifecycle (IsProcRunning, TerminateProc, WaitForProc via OpenProcess)", ANSI_TO_TCHAR( ARG_PROC_OPENPROCESS_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test case-insensitive file operations", ANSI_TO_TCHAR( ARG_CASE_SENSITIVITY_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test message box bug (too long strings)", ANSI_TO_TCHAR( ARG_MESSAGEBOX_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test directory watcher", ANSI_TO_TCHAR( ARG_DIRECTORY_WATCHER_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test per-thread singletons", ANSI_TO_TCHAR( ARG_THREAD_SINGLETON_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test (some) system information", ANSI_TO_TCHAR( ARG_SYSINFO_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test crash handling (pass '-logfatal' for testing Fatal logs)", ANSI_TO_TCHAR( ARG_CRASH_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test ensure handling (including ensureAlways and repeated ensures)", ANSI_TO_TCHAR( ARG_ENSURE_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test passing %%*s in a format string", ANSI_TO_TCHAR( ARG_STRINGPRECISION_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test APIs for dealing with dynamic libraries", ANSI_TO_TCHAR( ARG_DSO_TEST ));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test GMalloc->GetAllocationSize()", UTF8_TO_TCHAR(ARG_GET_ALLOCATION_SIZE_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test malloc for thread-safety and performance.", UTF8_TO_TCHAR(ARG_MALLOC_THREADING_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test by replaying a saved malloc history saved by -mallocsavereplay. Possible options: -replayfile=File, -stopafter=N (operation), -suppresserrors", UTF8_TO_TCHAR(ARG_MALLOC_REPLAY));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test thread priorities.", UTF8_TO_TCHAR(ARG_THREAD_PRIO_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test inline callstacks through ensures and a final crash.", UTF8_TO_TCHAR(ARG_INLINE_CALLSTACK_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test string allocations.", UTF8_TO_TCHAR(ARG_STRINGS_ALLOCATION_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test CreateGuid.", UTF8_TO_TCHAR(ARG_CREATEGUID_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test ThreadWalkStackAndDump and CaptureThreadBackTrace.", UTF8_TO_TCHAR(ARG_THREADSTACK_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test ExecProcess. Possible options: [-exe=executable] [-params=parameters]", UTF8_TO_TCHAR(ARG_EXEC_PROCESS_TEST));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test replacing open files. Start the instance with %ls and pass -file argument so it runs and keeps it open. This file is intended to be the target for '%ls' test.", 
		UTF8_TO_TCHAR(ARG_CMDLINE_FILE_KEEP_OPEN), UTF8_TO_TCHAR(ARG_CMDLINE_FILE_KEEP_OPEN), UTF8_TO_TCHAR(ARG_CMDLINE_FILE_REPLACE_OPEN));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test replacing open files. Run instance with %ls giving it a -filesource and -filetarget arguments. Target file (expected to be kept open by '%ls' test) will be replaced with the source.",
		UTF8_TO_TCHAR(ARG_CMDLINE_FILE_REPLACE_OPEN), UTF8_TO_TCHAR(ARG_CMDLINE_FILE_REPLACE_OPEN), UTF8_TO_TCHAR(ARG_CMDLINE_FILE_KEEP_OPEN));
	UE_LOGF(LogTestPAL, Warning, "  %ls: test replacing memory-mapped files within the same process. Creates a temp file, maps it into the memory, then attempts to replace with a new one.", UTF8_TO_TCHAR(ARG_CMDLINE_FILE_DELETE_MAPPED));

	if (PLATFORM_LINUX)
	{
		UE_LOGF(LogTestPAL, Warning, "  %ls: test WaitAndFork", UTF8_TO_TCHAR(ARG_FORK_TEST));
		UE_LOGF(LogTestPAL, Warning, "  %ls: test CmdlineParse", UTF8_TO_TCHAR(ARG_CMDLINE_PARSE_TEST));
	}

	UE_LOGF(LogTestPAL, Warning, "");
	UE_LOGF(LogTestPAL, Warning, "Pass one of those to run an appropriate test.");

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 1;
}

int32 main(int32 ArgC, char* ArgV[])
{
	TestPAL::CommandLine = TEXT("");

	for (int32 Option = 1; Option < ArgC; Option++)
	{
		TestPAL::CommandLine += TEXT(" ");
		FString Argument(ANSI_TO_TCHAR(ArgV[Option]));
		if (Argument.Contains(TEXT(" ")))
		{
			if (Argument.Contains(TEXT("=")))
			{
				FString ArgName;
				FString ArgValue;
				Argument.Split( TEXT("="), &ArgName, &ArgValue );
				Argument = FString::Printf( TEXT("%s=\"%s\""), *ArgName, *ArgValue );
			}
			else
			{
				Argument = FString::Printf(TEXT("\"%s\""), *Argument);
			}
		}
		TestPAL::CommandLine += Argument;
	}

	return MultiplexedMain(ArgC, ArgV);
}
