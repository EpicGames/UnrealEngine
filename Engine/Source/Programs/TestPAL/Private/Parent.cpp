// Copyright Epic Games, Inc. All Rights Reserved.

#include "Parent.h"
#include "TestPALLog.h"

FParent::FParent(int InNumTotalChildren, int InMaxChildrenAtOnce)
	:	NumTotalChildren(InNumTotalChildren)
	,	MaxChildrenAtOnce(InMaxChildrenAtOnce)
{

}

FProcHandle FParent::Launch(bool bDetached)
{
	// Launch the worker process
	int32 PriorityModifier = -1; // below normal

	uint32 WorkerId = 0;
	FString WorkerName = FPlatformProcess::ExecutableName(false);
	FProcHandle WorkerHandle = FPlatformProcess::CreateProc(*WorkerName, TEXT("proc-child"), bDetached, false, false, &WorkerId, PriorityModifier, NULL, NULL);
	if (!WorkerHandle.IsValid())
	{
		// If this doesn't error, the app will hang waiting for jobs that can never be completed
		UE_LOGF(LogTestPAL, Fatal, "Couldn't launch %ls! Make sure the file is in your binaries folder.", *WorkerName);
	}

	return WorkerHandle;
}

void FParent::Run()
{
	// test launching detached
	{
		for (int i = 0; i < 100; ++i)
		{
			UE_LOGF(LogTestPAL, Log, "Launching a detached child to see if we leak a zombie.");
			FProcHandle Child = Launch(true);
			
			FPlatformProcess::CloseProc(Child);
		}
	}

	// test stopping children prematurely
	{
		UE_LOGF(LogTestPAL, Log, "Launching a child to wait for it.");
		FProcHandle Child = Launch();

		UE_LOGF(LogTestPAL, Log, "Closing child's handle (FPlatformProcess::CloseProc)");
		FPlatformProcess::CloseProc(Child);
	}

	{
		UE_LOGF(LogTestPAL, Log, "Launching a child to terminate it.");
		FProcHandle Child = Launch();

		UE_LOGF(LogTestPAL, Log, "Sleeping for a bit to let the child ramp up.");
		FPlatformProcess::Sleep(0.1f);

		UE_LOGF(LogTestPAL, Log, "Terminating the child (FPlatformProcess::TerminateProc())");
		FPlatformProcess::TerminateProc(Child);

		UE_LOGF(LogTestPAL, Log, "Closing child's handle (FPlatformProcess::CloseProc)");
		FPlatformProcess::CloseProc(Child);
	}

	UE_LOGF(LogTestPAL, Log, "Proceeding to test multiple children.");

	// test normal working loop
	while (NumTotalChildren > 0)
	{
		// see if there are any new children to spawn
		while (Children.Num() < MaxChildrenAtOnce)
		{
			UE_LOGF(LogTestPAL, Log, "Launching a child (%d more to go).", NumTotalChildren - 1);
			FProcHandle Child = Launch();
			UE_LOGF(LogTestPAL, Log, "Launch successful");

			Children.Add(Child);
			--NumTotalChildren;
		}

		// sleep for a while
		FPlatformProcess::Sleep(0.5f);

		// see if any children have finished
		for (int ChildIdx = 0; ChildIdx < Children.Num();)
		{
			int32 ReturnCode = -1;
			FProcHandle ChildCopy = Children[ChildIdx];
			if (FPlatformProcess::GetProcReturnCode(ChildCopy, &ReturnCode))
			{
				// print its return code
				UE_LOGF(LogTestPAL, Log, "Child finished, return code %d", ReturnCode);

				FPlatformProcess::CloseProc(ChildCopy);
				Children.RemoveAt(ChildIdx);
			}
			else
			{
				++ChildIdx;
			}
		}
	}
}
