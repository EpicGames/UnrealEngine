// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformProcess.h"

class FParent
{
	/** Children to be controlled */
	TArray< FProcHandle > Children;

	/** Total children to spawn */
	int NumTotalChildren;

	/** Max children at once */
	int MaxChildrenAtOnce;

	/** Launches the child, returns the launched process handle. */
	FProcHandle Launch(bool bDetached = false);

public:

	/** Spawns the children one by one */
	void Run();

	FParent(int InNumTotalChildren, int InMaxChildrenAtOnce);
};
