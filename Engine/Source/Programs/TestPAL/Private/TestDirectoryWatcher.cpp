// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDirectoryWatcher.h"

#if USE_DIRECTORY_WATCHER

#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "LaunchEngineLoop.h"	// GEngineLoop
#include "Modules/ModuleManager.h"
#include "TestPALLog.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"

struct FChangeDetector
{
	void OnDirectoryChangedFunc(const FString& Title, const TArray<FFileChangeData>& FileChanges)
	{
		UE_LOGF(LogTestPAL, Display, "  -- %ls %d change(s) detected", *Title, static_cast<int32>(FileChanges.Num()));

		int ChangeIdx = 0;
		for (const auto& ThisEntry : FileChanges)
		{
			UE_LOGF(LogTestPAL, Display, "      Change %d: %ls was %ls",
				++ChangeIdx,
				*ThisEntry.Filename,
				ThisEntry.Action == FFileChangeData::FCA_Added ? TEXT("added") :
					(ThisEntry.Action == FFileChangeData::FCA_Removed ? TEXT("removed") :
						(ThisEntry.Action == FFileChangeData::FCA_Modified ? TEXT("modified") : TEXT("??? (unknown)")
						)
					)
			);
		}

		UE_LOGF(LogTestPAL, Display, "");
	}

	void OnDirectoryChanged(const TArray<FFileChangeData>& FileChanges)
	{
		OnDirectoryChangedFunc(TEXT("OnDirectoryChanged1"), FileChanges);
	}
	void OnDirectoryChanged2(const TArray<FFileChangeData>& FileChanges)
	{
		OnDirectoryChangedFunc(TEXT("OnDirectoryChanged2"), FileChanges);
	}
};

static void DoTick(IDirectoryWatcher* DirectoryWatcher, float Dt = 0.2f)
{
	DirectoryWatcher->Tick(Dt);
	FPlatformProcess::Sleep(Dt);
	DirectoryWatcher->Tick(Dt);
}

static void DoCommand(IDirectoryWatcher *DirectoryWatcher, const FString& Command, const FString& Arg)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (Command == TEXT("CreateDirectory"))
	{
		UE_LOGF(LogTestPAL, Display, "Creating DIRECTORY '%ls'", *Arg);
		verify(PlatformFile.CreateDirectory(*Arg));
	}
	else if (Command == TEXT("DeleteDirectory"))
	{
		UE_LOGF(LogTestPAL, Display, "Deleting DIRECTORY '%ls'", *Arg);
		verify(PlatformFile.DeleteDirectory(*Arg));
	}
	else if (Command == TEXT("CreateModifyFile"))
	{
		// create file
		UE_LOGF(LogTestPAL, Display, "Creating FILE '%ls'", *Arg);
		IFileHandle* DummyFile = PlatformFile.OpenWrite(*Arg);
		check(DummyFile);
		DoTick(DirectoryWatcher);

		// modify file
		UE_LOGF(LogTestPAL, Display, "Modifying FILE '%ls'", *Arg);
		uint8 Contents = 0x32;
		DummyFile->Write(&Contents, sizeof(Contents));
		DoTick(DirectoryWatcher);

		// close the file
		UE_LOGF(LogTestPAL, Display, "Closing FILE '%ls'", *Arg);
		delete DummyFile;
		DummyFile = nullptr;
		DoTick(DirectoryWatcher);
	}
	else if (Command == TEXT("DeleteFile"))
	{
		// delete file
		UE_LOGF(LogTestPAL, Display, "Deleting FILE '%ls'", *Arg);
		verify(PlatformFile.DeleteFile(*Arg));
		DoTick(DirectoryWatcher);
	}
	else
	{
		checkf(false, TEXT("Unknown command"));
	}

	DoTick(DirectoryWatcher);

	DirectoryWatcher->DumpStats();

}

/**
 * Kicks directory watcher test/
 */
int32 DirectoryWatcherTest(const TCHAR* CommandLine)
{
	FPlatformMisc::SetCrashHandler(NULL);
	FPlatformMisc::SetGracefulTerminationHandler();

	GEngineLoop.PreInit(CommandLine);
	UE_LOGF(LogTestPAL, Display, "Running directory watcher test.");

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	const FString TestDir = FString::Printf(TEXT("%s/DirectoryWatcherTest-%d"), FPlatformProcess::UserTempDir(), FPlatformProcess::GetCurrentProcessId());
	const FString SubtestDir = TestDir + TEXT("/subtest");

	if (PlatformFile.CreateDirectory(*TestDir) && PlatformFile.CreateDirectory(*SubtestDir))
	{
		FChangeDetector Detector, Detector2;
		FDelegateHandle DirectoryChangedHandle, DirectoryChangedHandle2;

		IDirectoryWatcher* DirectoryWatcher = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher")).Get();
		if (DirectoryWatcher)
		{
			/** Whether to include notifications for changes to actual directories (such as directories being created or removed). */
			//   IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree
			/** Whether changes in subdirectories need to be reported. */
			//   IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges
			uint32 Flags = IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges;

			auto Callback = IDirectoryWatcher::FDirectoryChanged::CreateRaw(&Detector, &FChangeDetector::OnDirectoryChanged);
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(TestDir, Callback, DirectoryChangedHandle, Flags);
			UE_LOGF(LogTestPAL, Display, "Registered callback for changes in '%ls'", *TestDir);

			auto Callback2 = IDirectoryWatcher::FDirectoryChanged::CreateRaw(&Detector2, &FChangeDetector::OnDirectoryChanged2);
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(SubtestDir, Callback2, DirectoryChangedHandle2, Flags);
			UE_LOGF(LogTestPAL, Display, "Registered callback 2 for changes in '%ls'", *SubtestDir);
		}
		else
		{
			UE_LOGF(LogTestPAL, Fatal, "Could not get DirectoryWatcher module");
		}

		DoTick(DirectoryWatcher);

		// create and remove directory
		DoCommand(DirectoryWatcher, "CreateDirectory", (TestDir + TEXT("/test")));
		DoCommand(DirectoryWatcher, "DeleteDirectory", (TestDir + TEXT("/test")));

		// create and remove in a sub directory
		DoCommand(DirectoryWatcher, "CreateDirectory", (SubtestDir + TEXT("/blah")));

		UE_LOGF(LogTestPAL, Display, "Moving DIRECTORY '%ls' -> '%ls'", *(SubtestDir + TEXT("/blah")), *(SubtestDir + TEXT("/blah2")));
		verify(PlatformFile.MoveFile(*(SubtestDir + TEXT("/blah2")), *(SubtestDir + TEXT("/blah"))));
		DoTick(DirectoryWatcher);

		DoCommand(DirectoryWatcher, "DeleteDirectory", (SubtestDir + TEXT("/blah2")));

		DoCommand(DirectoryWatcher, "CreateModifyFile", TestDir + TEXT("/test file.bin"));
		DoCommand(DirectoryWatcher, "DeleteFile", TestDir + TEXT("/test file.bin"));

		// now the same in a grandchild directory
		{
			FString GrandChildDir = SubtestDir + TEXT("/grandchild");

			DoCommand(DirectoryWatcher, "CreateDirectory", GrandChildDir);
			DoCommand(DirectoryWatcher, "CreateModifyFile", (GrandChildDir + TEXT("/test file.bin")));
			DoCommand(DirectoryWatcher, "DeleteFile", (GrandChildDir + TEXT("/test file.bin")));
			DoCommand(DirectoryWatcher, "DeleteDirectory", GrandChildDir);
		}

		// remove main dirs as another test case
		DoCommand(DirectoryWatcher, "DeleteDirectory", SubtestDir);
		DoCommand(DirectoryWatcher, "DeleteDirectory", TestDir);

		// Clean up
		verify(DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(TestDir, DirectoryChangedHandle));
		verify(DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(SubtestDir, DirectoryChangedHandle2));

		// Unregister requests get moved to a list for deletion during tick
		DoTick(DirectoryWatcher);

		UE_LOGF(LogTestPAL, Display, "End of test");
	}
	else
	{
		UE_LOGF(LogTestPAL, Fatal, "Could not create test directory %ls.", *TestDir);
	}

	FEngineLoop::AppPreExit();
	FEngineLoop::AppExit();
	return 0;
}

#endif /* USE_DIRECTORY_WATCHER */
