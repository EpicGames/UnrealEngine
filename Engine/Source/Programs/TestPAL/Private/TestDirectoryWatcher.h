// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define USE_DIRECTORY_WATCHER (PLATFORM_WINDOWS || PLATFORM_MAC || PLATFORM_LINUX)

int32 DirectoryWatcherTest(const TCHAR* CommandLine);
