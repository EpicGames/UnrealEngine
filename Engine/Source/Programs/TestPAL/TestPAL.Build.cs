// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TestPAL : ModuleRules
{
	public TestPAL(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"ApplicationCore",
				"Projects",
			}
		);

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.Platform == UnrealTargetPlatform.Mac ||
			Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DirectoryWatcher"
				}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.Linux || Target.Platform == UnrealTargetPlatform.LinuxArm64)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"SDL3",
				}
			);
		}
	}
}
