// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class TestPALTarget : TargetRules
{
	public TestPALTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "TestPAL";

		// Lean and mean
		bBuildDeveloperTools = false;

		// Compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = false;

		// Logs are still useful to print the results
		bUseLoggingInShipping = true;

		// Make a console application under Windows, so entry point is main() everywhere
		bIsBuildingConsoleApplication = true;

		if (Target.Platform == UnrealTargetPlatform.Linux && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			GlobalDefinitions.Add("COMPILE_FORK_PAGE_PROTECTOR=1");
		}
	}
}
